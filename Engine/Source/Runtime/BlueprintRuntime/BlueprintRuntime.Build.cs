// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class BlueprintRuntime : ModuleRules
	{
        public BlueprintRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				    "CoreUObject",
				}
			);
		}
	}
}
