// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintRuntime.h"
#include "UObject/Class.h"
#include "BlueprintRuntimeSettings.h"
#include "Blueprint/BlueprintSupport.h"

#define LOCTEXT_NAMESPACE "BlueprintRuntime"

class FBlueprintRuntime : public IBlueprintRuntime
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual void PropagateWarningSettings() override;
	virtual UBlueprintRuntimeSettings* GetMutableBlueprintRuntimeSettings() override;
};
IMPLEMENT_MODULE(FBlueprintRuntime, BlueprintRuntime)

void FBlueprintRuntime::StartupModule()
{
}

void FBlueprintRuntime::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

void FBlueprintRuntime::PropagateWarningSettings()
{
	const UBlueprintRuntimeSettings* BlueprintRuntimeSettings = GetDefault<UBlueprintRuntimeSettings>();
	// Propagate current settings to the core blueprint runtime, which can not easily
	// leverage the reflection system:
	TArray<FName> WarningsToTreatAsErrors;
	TArray<FName> WarningsToSuppress;

	for (const FBlueprintWarningSettings& Setting : BlueprintRuntimeSettings->WarningSettings)
	{
		if(Setting.WarningBehavior == EBlueprintWarningBehavior::Error)
		{
			WarningsToTreatAsErrors.Add(Setting.WarningIdentifier);
		}
		else if( Setting.WarningBehavior == EBlueprintWarningBehavior::Suppress)
		{
			WarningsToSuppress.Add(Setting.WarningIdentifier);
		}
	}
	FBlueprintSupport::UpdateWarningBehavior(WarningsToTreatAsErrors, WarningsToSuppress);
}

UBlueprintRuntimeSettings* FBlueprintRuntime::GetMutableBlueprintRuntimeSettings()
{
	return GetMutableDefault<UBlueprintRuntimeSettings>();
}


#undef LOCTEXT_NAMESPACE
