// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintRuntimeSettings.generated.h"

class Error;

UENUM()
enum class EBlueprintWarningBehavior : uint8
{
	Warn,
	Error,
	Suppress,
};

USTRUCT()
struct FBlueprintWarningSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName WarningIdentifier;

	UPROPERTY()
	FText WarningDescription;

	UPROPERTY()
	EBlueprintWarningBehavior WarningBehavior = EBlueprintWarningBehavior::Warn;
};

/**
* Implements the settings for the BlueprintRuntime module
*/
UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UBlueprintRuntimeSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, config, Category = Warnings)
	TArray<FBlueprintWarningSettings> WarningSettings;
};
